// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UnrealMultiUserSlateServerRun.h"
#include "LaunchEngineLoop.h"
#include "UnixCommonStartup.h"

int main(int argc, char *argv[])
{
	return CommonUnixMain(argc, argv, &RunUnrealMultiUserServer, [] { } );
}
