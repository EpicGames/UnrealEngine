// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealMultiUserSlateServerRun.h"
#include "Mac/MacProgramDelegate.h"
#include "LaunchEngineLoop.h"

int main( int argc, char *argv[] )
{
	return [MacProgramDelegate mainWithArgc:argc argv:argv programMain:RunUnrealMultiUserServer programExit:FEngineLoop::AppExit];
}
