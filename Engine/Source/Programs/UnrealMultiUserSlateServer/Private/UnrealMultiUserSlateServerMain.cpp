// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"
#include "RequiredProgramMainCPPInclude.h"

IMPLEMENT_APPLICATION(UnrealMultiUserSlateServer, "UnrealMultiUserSlateServer");
