// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealMultiUserSlateServerRun.h"

#include "ConcertSettings.h"
#include "ConcertSyncServerLoop.h"

#include "IMultiUserServerModule.h"

namespace UE::UnrealMultiUserServer
{
	static void SetupSlate(FConcertSyncServerLoopInitArgs& ServerLoopInitArgs)
	{
		ServerLoopInitArgs.bShowConsole = false;
		ServerLoopInitArgs.PreInitServerLoop.AddLambda([&ServerLoopInitArgs]()
		{
			TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("MultiUserServer"));
			if (!Plugin || !Plugin->IsEnabled())
			{
				UE_LOGF(LogSyncServer, Error, "The 'MultiUserServer' plugin is disabled.");
			}
			else
			{
				IMultiUserServerModule::Get().InitSlateForServer(ServerLoopInitArgs);
			}
		});
	}
}

int32 RunUnrealMultiUserServer(const TCHAR* CommandLine)
{
	FString Role(TEXT("MultiUser"));
	FConcertSyncServerLoopInitArgs ServerLoopInitArgs;
	ServerLoopInitArgs.SessionFlags = EConcertSyncSessionFlags::Default_MultiUserSession;
	ServerLoopInitArgs.ServiceRole = Role;
	ServerLoopInitArgs.ServiceFriendlyName = TEXT("Multi-User Editing Server");
	ServerLoopInitArgs.GetFileSharingServiceFunc = [Role]() -> TSharedPtr<IConcertFileSharingService>
	{
		return IConcertSyncServerModule::Get().CreateFileSharingService(Role);
	};

	ServerLoopInitArgs.GetServerConfigFunc = [Role]() -> const UConcertServerConfig*
	{
		UConcertServerConfig* ServerConfig = IConcertSyncServerModule::Get().ParseServerSettings(FCommandLine::Get());
		if (ServerConfig->WorkingDir.IsEmpty())
		{
			ServerConfig->WorkingDir = FPaths::ProjectIntermediateDir() / Role;
		}
		if (ServerConfig->ArchiveDir.IsEmpty())
		{
			ServerConfig->ArchiveDir = FPaths::ProjectSavedDir() / Role;
		}
		return ServerConfig;
	};

	UE::UnrealMultiUserServer::SetupSlate(ServerLoopInitArgs);
	return ConcertSyncServerLoop(CommandLine, ServerLoopInitArgs);
}
