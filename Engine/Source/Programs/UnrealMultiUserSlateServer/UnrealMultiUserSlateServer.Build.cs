// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnrealMultiUserSlateServer : ModuleRules
{
	public UnrealMultiUserSlateServer(ReadOnlyTargetRules Target) : base(Target)
	{
		//ShortName = "MU-SlateServer";
		bRequiresPlatformSDK = true;
		
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Concert",
				"ConcertServer",
				"TraceInsights",
				"ApplicationCore",					// for LaunchEngineLoop.cpp dependency
				"Projects",							// for LaunchEngineLoop.cpp dependency
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Launch",
				"ConcertSyncCore",
				"ConcertSyncServer",
				"MultiUserServer",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"ConcertSyncServer",
				"MultiUserServer"
			}
		);
		
		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"UnixCommonStartup"
				}
			);
		}

		ShortName = "UEMultiUsrSltServ";
	}
}
