// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class UnrealMultiUserSlateServerTarget : TargetRules
{
	public UnrealMultiUserSlateServerTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Modular;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "UnrealMultiUserSlateServer";
		AdditionalPlugins.Add("UdpMessaging");
		AdditionalPlugins.Add("QuicMessaging");
		AdditionalPlugins.Add("ConcertSyncServer");
		AdditionalPlugins.Add("MultiUserServer");

		// This app compiles against Core/CoreUObject, but not the Engine or Editor, so compile out Engine and Editor references from Core/CoreUObject
		bCompileAgainstCoreUObject = true;
		bCompileAgainstEngine = false;
		bBuildWithEditorOnlyData = false;

		// Enable Developer plugins (like Concert!)
		bCompileWithPluginSupport = true;
		bBuildDeveloperTools = true;
		bIsBuildingConsoleApplication = false;

		GlobalDefinitions.Add("UE_LOG_CONCERT_DEBUG_VERBOSITY_LEVEL=Log");
		bEnableTrace = true;
	}
}
